// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class AudioFormatRad : ModuleRules
{
	// NDA platforms will override this.
    protected virtual string[] GetRadAudioPlatformString(ReadOnlyTargetRules Target, out bool bIsWindowsLib)
    {
        bIsWindowsLib = false; // most platforms are not windows style.
        if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
        {
            bIsWindowsLib = true;
            if (Target.Architecture == UnrealArch.Arm64)
            {
                return new string[] { "winarm64" };
            }
            if (Target.Architecture == UnrealArch.X64)
            {
                return new string[] { "win64" };
            }
            if (Target.Architecture == UnrealArch.Arm64ec)
            {
                return new string[] { "win64" };
            }
            System.Console.WriteLine("Unknown windows architecture ({0} in AudioFormatRad.Build.cs", Target.Architecture);
            return null;
        }
        if (Target.Platform == UnrealTargetPlatform.Linux)
        {
            if (Target.Architecture == UnrealArch.Arm64)
            {
                return new string[] { "linuxarm64" };
            }
            if (Target.Architecture == UnrealArch.X64)
            {
                return new string[] { "linux64" };
            }
            
            System.Console.WriteLine("Unknown linux architecture ({0} in AudioFormatRad.Build.cs", Target.Architecture);
            return null;
        }
        if (Target.Platform == UnrealTargetPlatform.LinuxArm64)
        {
            return new string[] { "linuxarm64" };
        }
        if (Target.Platform == UnrealTargetPlatform.Mac)
        {
            return new string[] { "osx" }; // arm64 and x64 are lipod together
        }

        return null;
    }


	public AudioFormatRad(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Engine"
			}
		);

		PublicSystemIncludePaths.Add(Path.Combine(ModuleDirectory, "..", "..", "Runtime", "RadAudioCodec", "SDK", "Include"));

		PublicDefinitions.Add("RADA_WRAP=UERA");

        bool bIsWindowsLib = false;
        string[] ArchitectureStrings = GetRadAudioPlatformString(Target, out bIsWindowsLib);
		if (ArchitectureStrings is not null)
		{
			string BasePath = Path.Combine(ModuleDirectory, "..", "..", "Runtime", "RadAudioCodec", "SDK", "Lib");
			foreach (string Architecture in ArchitectureStrings)
			{
				// radaudio_ is the codec.
				// rada_ is the container.
				if (bIsWindowsLib)
				{
					PublicAdditionalLibraries.Add(Path.Combine(BasePath, "radaudio_decoder_" + Architecture + ".lib"));
					PublicAdditionalLibraries.Add(Path.Combine(BasePath, "radaudio_encoder_" + Architecture + ".lib"));
				}
				else
				{
					PublicAdditionalLibraries.Add(Path.Combine(BasePath, "libradaudio_decoder_" + Architecture + ".a"));
					PublicAdditionalLibraries.Add(Path.Combine(BasePath, "libradaudio_encoder_" + Architecture + ".a"));
				}
			}
		}
	}
}
