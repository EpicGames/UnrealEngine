// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "Interfaces/IAudioFormat.h"
#include "Interfaces/IAudioFormatModule.h"
#include "HAL/Platform.h"

#include "rada_file_header.h"
#include "rada_encode.h"
#include "rada_decode.h"

static const FName NAME_RADA(TEXT("RADA"));

DEFINE_LOG_CATEGORY_STATIC(LogAudioFormatRad, Display, All);

namespace AudioFormatRadPrivate
{
	static uint8 GetCompressionLevelFromQualityIndex(const int32 InQualityIndex) 
	{
		// RAD is tuned for 5, so we map almost everything to that, with some extremes 
		// for manipulating edge cases. Project defaults vary from 40-80, but we make it symmetric:
		uint8 RadLevel = 5;
		if (InQualityIndex < 20) // 1..19 -> 1..4
		{
			RadLevel = FMath::RoundToInt(FMath::GetMappedRangeValueClamped(FVector2d(1, 19), FVector2d(1, 4), InQualityIndex));
		}
		else if (InQualityIndex > 80) // 81..100 -> 6..9
		{
			RadLevel = FMath::RoundToInt(FMath::GetMappedRangeValueClamped(FVector2d(81, 100), FVector2d(6, 9), InQualityIndex));
		}

		return RadLevel;
	}
}

/**
 * IAudioFormat, audio compression abstraction
**/
class FAudioFormatRad : public IAudioFormat
{
	enum
	{
		/** Version for RAD Audio format, this becomes part of the DDC key. */
		UE_AUDIO_RAD_VER = 5,
	};

public:
	virtual bool AllowParallelBuild() const override
	{
		return true;
	}

	virtual uint16 GetVersion(FName Format) const override
	{
		check(Format == NAME_RADA);
		return UE_AUDIO_RAD_VER;
	}

	virtual void GetSupportedFormats(TArray<FName>& OutFormats) const override
	{
		OutFormats.Add(NAME_RADA);
	}
	
	virtual TConstArrayView<int32> GetSupportedSampleRates() const override
	{
		static constexpr int32 Supported[] = { 48000, 44100, 32000, 24000 };
		return Supported;
	}

	static void* RadAlloc(const size_t Bytes)
	{
		return FMemory::Malloc(Bytes, 16);
	}
	static void RadFree(void* Ptr)
	{
		FMemory::Free(Ptr);
	}

	virtual bool Cook(FName InFormat, const TArray<uint8>& InSrcBuffer, FSoundQualityInfo& InQualityInfo, TArray<uint8>& OutCompressedDataStore) const override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAudioFormatRad::Cook);
		check(InFormat == NAME_RADA);

		#ifdef RADA_BUILD_VERSION
		if (RadAGetBuildVersion() != RADA_BUILD_VERSION)
		{
			UE_LOGF(LogAudioFormatRad, Error, "Header/library mismatch with Rad Audio! Header: %u, Library: %u - verify sync is correct", RADA_BUILD_VERSION, RadAGetBuildVersion());
			return false;
		}
		#endif


		const uint8 CompressionLevel = AudioFormatRadPrivate::GetCompressionLevelFromQualityIndex(InQualityInfo.Quality);
		
		// If we're going to embed the seek-table in the stream, use -1 to give the largest table we can produce.
		// \todo not sure what this was doing. It looks like it's just passing in what would get generated, which
		// seem like just means it's never capping it, so we can just pass the max?
		const uint16 MaxSeektableSize = 65535;
		
		uint8* CompressedData = nullptr;
		uint64_t CompressedDataLen = 0;

		// For the moment we don't support the fancy looping behavior as it's not plumbed down in UE - we just always encode
		// "normally".
		uint8_t RadCompressError =  EncodeRadAFile(
			(void*)InSrcBuffer.GetData(), InSrcBuffer.Num(), 
			InQualityInfo.SampleRate, InQualityInfo.NumChannels, CompressionLevel, 
			0, 1, MaxSeektableSize, RadAlloc, RadFree, 
			(void**)&CompressedData, &CompressedDataLen);

		if (RadCompressError)
		{
			UE_LOGF(LogAudioFormatRad, Warning, "Failed to encode RAD Audio: %s",  RadAErrorString(RadCompressError));
			if (RadCompressError == RADA_COMPRESS_ERROR_RATE)
			{
				UE_LOGF(LogAudioFormatRad, Warning, "Sample rate provided: %u - please reimport at a valid sample rate.", InQualityInfo.SampleRate);
			}
			else if (RadCompressError == RADA_COMPRESS_ERROR_CHANS)
			{
				UE_LOGF(LogAudioFormatRad, Warning, "Channels provided: %u, max channels allowed: %d", InQualityInfo.NumChannels, RADA_MAX_CHANS);
			}
			return false;
		}

		// copy to unreal structures.
		OutCompressedDataStore.Empty();
		OutCompressedDataStore.Append((uint8*)CompressedData, CompressedDataLen);
		RadFree(CompressedData);
		return OutCompressedDataStore.Num() > 0;
	}

	virtual bool CookSurround(FName InFormat, const TArray<TArray<uint8> >& InSrcBuffers, FSoundQualityInfo& InQualityInfo, TArray<uint8>& OutCompressedDataStore) const override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAudioFormatRad::CookSurround);
		check(InFormat == NAME_RADA);

		#ifdef RADA_BUILD_VERSION
		if (RadAGetBuildVersion() != RADA_BUILD_VERSION)
		{
			UE_LOGF(LogAudioFormatRad, Error, "Header/library mismatch with Rad Audio! Header: %u, Library: %u - verify sync is correct", RADA_BUILD_VERSION, RadAGetBuildVersion());
			return false;
		}
		#endif

		//
		// CookSurround passes us a bunch of mono buffers, but RAD audio wants a standard 
		// interleaved buffer
		//
		TArray<uint8> InterleavedSrcBuffers;
		InterleavedSrcBuffers.AddUninitialized(InSrcBuffers[0].Num() * InSrcBuffers.Num());
		int16* Dest = (int16*)InterleavedSrcBuffers.GetData();
		uint32 ChannelCount = InSrcBuffers.Num();
		uint32 FrameCount = InSrcBuffers[0].Num() / sizeof(int16);
		for (uint32 FrameIndex = 0; FrameIndex < FrameCount; FrameIndex++)
		{
			for (uint32 ChannelIndex = 0; ChannelIndex < ChannelCount; ChannelIndex++)
			{
				int16* Src = (int16*)InSrcBuffers[ChannelIndex].GetData();
				Dest[FrameIndex * ChannelCount + ChannelIndex] = Src[FrameIndex];
			}
		}

		return Cook(NAME_RADA, InterleavedSrcBuffers, InQualityInfo, OutCompressedDataStore);
	}

	// AFAICT this function is never called.
	virtual int32 Recompress(FName Format, const TArray<uint8>& SrcBuffer, FSoundQualityInfo& QualityInfo, TArray<uint8>& OutBuffer) const override
	{
		return 0;
	}

	virtual int32 GetMinimumSizeForInitialChunk(FName Format, const TArray<uint8>& SrcBuffer) const override
	{
		// We must have an initial chunk large enough for the header and the seek table, if present.

		// Exclude any seek table entries in our size when we are using streaming seek tables.
		bool bIncludeSeekTableSize = !RequiresStreamingSeekTable();
		
		const RadAFileHeader* FileHeader = RadAGetFileHeader(SrcBuffer.GetData(), SrcBuffer.Num());
		if (FileHeader == 0)
		{
			UE_LOGF(LogAudioFormatRad, Error, "Invalid buffer passed to GetMinimumSizeForInitialChunk (size=%d)", SrcBuffer.Num());
			return 0;
		}

		int64_t BytesToFirstBlock = RadAGetBytesToOpen(FileHeader);
		if (!bIncludeSeekTableSize)
		{
			BytesToFirstBlock -= RadAGetSeekTableSizeOnDisk(FileHeader);
			BytesToFirstBlock -= sizeof(RadASeekTableHeader);
			check(BytesToFirstBlock >= 0);
		}

		return IntCastChecked<int32>(BytesToFirstBlock);
	}

	// Takes in a compressed file and splits it into stream size chunks. AFAICT this is supposed to accumulate frames
	// until the chunk size is reached, then spit out a block.
	virtual bool SplitDataForStreaming(const TArray<uint8>& InSrcBuffer, TArray<TArray<uint8>>& OutBuffers, const int32 InMaxInitialChunkSize, const int32 InMaxChunkSize) const override
	{
		// This should not be called if we require a streaming seek-table. 
		if (!ensure(RequiresStreamingSeekTable()==false))
		{
			return false;
		}

		uint8 const* Source = InSrcBuffer.GetData();
		uint32 SourceLen = InSrcBuffer.Num();
		uint8 const* SourceEnd = Source + SourceLen;

		uint8 const* ChunkStart = Source;
		uint8 const* Current = Source;

		const RadAFileHeader* FileHeader = RadAGetFileHeader(Source, SourceLen);
		if (FileHeader == 0)
		{
			UE_LOGF(LogAudioFormatRad, Error, "Buffer provided to SplitDataForStreaming is not a RADA file!");
			return false;
		}

		// We need to open the decoder in order to get block sizes from chunks.
		uint32_t ContainerMemoryRequried = 0;
		if (RadAGetMemoryNeededToOpen(Source, SourceLen, &ContainerMemoryRequried) != 0)
		{
			// Should never happen as we were able to get the header above - if it did the data is corrupt.
			UE_LOGF(LogAudioFormatRad, Error, "Couldn't figure memory required to open Rada decoder - invalid file.");
			return false;
		}

		TArray<uint8> ContainerBytes;
		ContainerBytes.AddUninitialized(ContainerMemoryRequried);
		RadAContainer* Container = (RadAContainer*)ContainerBytes.GetData();
		if (RadAOpenDecoder(Source, SourceLen, Container, ContainerMemoryRequried) == 0)
		{
			UE_LOGF(LogAudioFormatRad, Error, "Couldn't open Rada decoder - invalid file.");
			return false;
		}

		Current += RadAGetBytesToOpen(FileHeader);

		int32 ChunkLimitBytes = InMaxInitialChunkSize;
		for (;;)
		{
			if (Current >= SourceEnd)
			{
				// Done with the file.
				check(Current == SourceEnd);
				Current = SourceEnd;
				break;
			}

			uint32_t BlockSize = 0;
			RadAExamineBlockResult Result = RadAExamineBlock(Container, Current, SourceEnd - Current, &BlockSize);

			if (Result != RadAExamineBlockResult::Valid)
			{
				UE_LOGF(LogAudioFormatRad, Error, "Couldn't parse rada block in file, offset %d", (uint32_t)(SourceEnd - Current));
				return false;
			}
			// Since we passed the Examine check we know the block fits in our memory and isn't corrupted.

			if ((Current - ChunkStart) + BlockSize >= ChunkLimitBytes)
			{
				// can't add this chunk, emit.
				OutBuffers.Emplace(ChunkStart, Current - ChunkStart);
				ChunkStart = Current;
				ChunkLimitBytes = InMaxChunkSize;

				// retry.
				continue;
			}

			Current += BlockSize;
		}

		// emit any remainder chunks
		if (SourceEnd - ChunkStart)
		{
			// emit this chunk
			OutBuffers.Emplace(ChunkStart, SourceEnd - ChunkStart);
		}

		return true;
	}
	
	virtual bool RequiresStreamingSeekTable() const override
	{
		return true; // Toggling this will require a version bump. (?? does that mean it's not encoded in the ddc key???)
	}

	virtual bool ExtractSeekTableForStreaming(TArray<uint8>& InOutBuffer, IAudioFormat::FSeekTable& OutSeekTable) const override
	{
		// This should only be called if we require a streaming seek-table. 
		if (!ensure(RequiresStreamingSeekTable()))
		{
			return false;
		}

		const RadAFileHeader* FileHeader = RadAGetFileHeader(InOutBuffer.GetData(), InOutBuffer.Num());
		if (FileHeader == nullptr)
		{
			return false;
		}

		const RadASeekTableHeader* SeekHeader = RadAGetSeekTableHeader(InOutBuffer.GetData(), InOutBuffer.Num());
		if (SeekHeader == nullptr)
		{
			return false;
		}

		uint32_t SeekTableOffset = RadAGetOffsetToSeekTable(FileHeader);
		uint32_t SeekTableSizeBytes = RadAGetSeekTableSizeOnDisk(FileHeader);
		uint64_t SizeNeededForSeekTable = SeekTableOffset + SeekTableSizeBytes;
		if (SizeNeededForSeekTable > TNumericLimits<uint32>::Max())
		{
			return false;
		}
		if (SizeNeededForSeekTable > InOutBuffer.Num())
		{
			// Not enough space for seek table in source...? Should never hit this sense we are called right after encode!
			return false;
		}

		uint8_t* SeekTableData = InOutBuffer.GetData() + SeekTableOffset;

		OutSeekTable.Offsets.SetNum(FileHeader->seek_table_entry_count);
		OutSeekTable.Times.SetNum(FileHeader->seek_table_entry_count);

		size_t SeekTableSizeBytesConsumed = 0;
		SeekTableEnumerationState EnumState;
		RadASeekTableReturn DecodeResult = RadADecodeSeekTable(
			FileHeader, SeekHeader, SeekTableData, SeekTableSizeBytes, false, 
			&EnumState, (uint8_t*)OutSeekTable.Times.GetData(), (uint8_t*)OutSeekTable.Offsets.GetData(), 
			&SeekTableSizeBytesConsumed);

		if (DecodeResult != RadASeekTableReturn::Done)
		{
			UE_LOGF(LogAudioFormatRad, Error, "Failed to decode seek table for streaming: result = %d", EnumToUnderlyingType(DecodeResult));
			return false;
		}

		// Check that the last block which spans the last offset in the table and end of file
		// is a reasonable size.
		if (!ensure(InOutBuffer.Num() - OutSeekTable.Offsets.Last() < 1024*1024))
		{
			return false;
		}

		// Strip the seek-table from the buffer now we've copied it.
		size_t SizeAfterStripping = RadAStripSeekTable(InOutBuffer.GetData(), InOutBuffer.Num());
		InOutBuffer.SetNum((int32)SizeAfterStripping);

		// The byte offsets we got include the seek table data in the stream, so subtract off of each one
		int32_t seek_table_bytes_to_remove = sizeof(RadASeekTableHeader) + SeekTableSizeBytes;
		for (uint32& ByteOffset : OutSeekTable.Offsets)
		{
			ByteOffset -= seek_table_bytes_to_remove;
		}

		return true;
	}
};

class FAudioPlatformRadModule final : public IAudioFormatModule
{
private:
	FAudioFormatRad* RadEncoder = nullptr;

public:
	virtual ~FAudioPlatformRadModule() override {}
	
	virtual IAudioFormat* GetAudioFormat() override
	{
		return RadEncoder;
	}
	virtual void StartupModule() override
	{
		#ifdef RADA_BUILD_VERSION
		uint32_t LibraryBuildVersion = RadAGetBuildVersion();
		if (LibraryBuildVersion != RADA_BUILD_VERSION)
		{
			UE_LOGF(LogAudioFormatRad, Error, "Header/library mismatch with Rad Audio! Header: %u, Library: %u - verify sync is correct", RADA_BUILD_VERSION, LibraryBuildVersion);
		}
		#endif

		RadEncoder = new FAudioFormatRad();
	}
	virtual void ShutdownModule() override
	{
		delete RadEncoder;
		RadEncoder = nullptr;
	}
};

IMPLEMENT_MODULE( FAudioPlatformRadModule, AudioFormatRad);
